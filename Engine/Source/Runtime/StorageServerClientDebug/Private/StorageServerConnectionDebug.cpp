// Copyright Epic Games, Inc. All Rights Reserved.

#include "StorageServerConnectionDebug.h"

#include "Debug/DebugDrawService.h"
#include "Engine/GameEngine.h"
#include "Engine/Texture2D.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Templates/UniquePtr.h"
#include "StorageServerClientModule.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "Misc/ScopeLock.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ConfigCacheIni.h"

#if !UE_BUILD_SHIPPING

CSV_DEFINE_CATEGORY(ZenServerStats, true);

CSV_DEFINE_STAT(ZenServerStats, ThroughputMbps);
CSV_DEFINE_STAT(ZenServerStats, MaxReqThroughputMbps);
CSV_DEFINE_STAT(ZenServerStats, MinReqThroughputMbps);
CSV_DEFINE_STAT(ZenServerStats, RequestCountPerSec);

TRACE_DECLARE_UNCHECKED_FLOAT_COUNTER(ZenClient_ThroughputMbps,       TEXT("ZenClient/ThroughputMbps (decompressed)"));
TRACE_DECLARE_UNCHECKED_FLOAT_COUNTER(ZenClient_MaxReqThroughputMbps, TEXT("ZenClient/MaxReqThroughputMbps (decompressed)"));
TRACE_DECLARE_UNCHECKED_FLOAT_COUNTER(ZenClient_MinReqThroughputMbps, TEXT("ZenClient/MinReqThroughputMbps (decompressed)"));
TRACE_DECLARE_UNCHECKED_INT_COUNTER(  ZenClient_RequestCountPerSec,   TEXT("ZenClient/RequestCountPerSec"));

static bool GZenShowIndicator = true;
static FAutoConsoleVariableRef CVarShowZenIndicator(
	TEXT("zen.indicator.show"),
	GZenShowIndicator,
	TEXT("Show on-screen indicator when Zen streaming is active"),
	ECVF_Default
);

static bool GZenShowDebugMessage = false;
static FAutoConsoleVariableRef CVarShowZenDebugOnScreenMessage(
	TEXT("zen.onscreenmessage"),
	GZenShowDebugMessage,
	TEXT("Show an on-screen message with zen streaming stats"),
	ECVF_Default
);

static float GZenIndicatorPosX = 0.01f;
static FAutoConsoleVariableRef CVarZenIndicatorPosX(
	TEXT("zen.indicator.x"),
	GZenIndicatorPosX,
	TEXT("Zen on-screen indicator position (horizontal)"),
	ECVF_Default
);

static float GZenIndicatorPosY = 0.8f;
static FAutoConsoleVariableRef CVarZenIndicatorPosY(
	TEXT("zen.indicator.y"),
	GZenIndicatorPosY,
	TEXT("Zen on-screen indicator position (vertical)"),
	ECVF_Default
);

static float GZenIndicatorFadeTime = -1.f;
static FAutoConsoleVariableRef CVarZenIndicatorFadeTime(
	TEXT("zen.indicator.fadetime"),
	GZenIndicatorFadeTime,
	TEXT("Zen on-screen indicator fade time in seconds"),
	ECVF_Default
);

static constexpr float IndicatorFadeSpeed = 5.f;

static float GZenIndicatorAlpha = 0.5f;
static FAutoConsoleVariableRef CVarZenIndicatorAlpha(
	TEXT("zen.indicator.alpha"),
	GZenIndicatorAlpha,
	TEXT("Zen on-screen indicator transparency"),
	ECVF_Default
);

static bool GZenShowGraphs = false;
static FAutoConsoleVariableRef CVarZenShowGraphs(
	TEXT("zen.showgraphs"),
	GZenShowGraphs,
	TEXT("Show ZenServer Stats Graph"),
	ECVF_Default
);

static bool GZenShowStats = true;
static FAutoConsoleVariableRef CVarZenShowStats(
	TEXT("zen.showstats"),
	GZenShowStats,
	TEXT("Show ZenServer Stats"),
	ECVF_Default
);


namespace
{
	static constexpr int    OneMinuteSeconds = 60;
	static constexpr double WidthSeconds = OneMinuteSeconds * 0.25;

	constexpr float			ZenIconPadding = 8.f;
	constexpr float			ZenIndicatorTextWidth = 256;
}


bool FStorageServerConnectionDebug::OnTick(float)
{
	FScopeLock Lock(&CS);

	static constexpr double FrameSeconds = 1.0;
	
	double StatsTimeNow = FPlatformTime::Seconds();
	double Duration = StatsTimeNow - UpdateStatsTime;
	IndicatorElapsedTime += StatsTimeNow - IndicatorLastTime;
	IndicatorLastTime = StatsTimeNow;

	//Persistent debug message and CSV stats
	if (Duration > UpdateStatsTimer)
	{
		UpdateStatsTime = StatsTimeNow;

		IStorageServerPlatformFile::FConnectionStats Stats;
		StorageServerPlatformFile->GetAndResetConnectionStats(Stats);
		if (Stats.MaxRequestThroughput > Stats.MinRequestThroughput)
		{
			MaxReqThroughput = Stats.MaxRequestThroughput;
			MinReqThroughput = Stats.MinRequestThroughput;
		
			Throughput = ((double)(Stats.AccumulatedBytes * 8) / Duration) / 1000000.0; //Mbps
			ReqCount = ceil((double)Stats.RequestCount / Duration);
		}

		if (GZenShowDebugMessage && GEngine)
		{
			FString ZenConnectionDebugMsg;
			ZenConnectionDebugMsg = FString::Printf(TEXT("ZenServer %s from %s [%.2fMbps]"), UE::IsUsingZenPakFileStreaming() ? TEXT("pak streaming") : TEXT("streaming"), * HostAddress, Throughput);
			GEngine->AddOnScreenDebugMessage((uint64)this, UpdateStatsTimer, FColor::White, ZenConnectionDebugMsg, false);
		}
		
		History.push_back({ StatsTimeNow, MaxReqThroughput, MinReqThroughput, Throughput, ReqCount });

		TRACE_COUNTER_SET(ZenClient_ThroughputMbps,       Throughput);
		TRACE_COUNTER_SET(ZenClient_MaxReqThroughputMbps, MaxReqThroughput);
		TRACE_COUNTER_SET(ZenClient_MinReqThroughputMbps, MinReqThroughput);
		TRACE_COUNTER_SET(ZenClient_RequestCountPerSec,   ReqCount);
	}

	while (!History.empty() && StatsTimeNow - History.front().Time > WidthSeconds)
	{
		History.erase(History.begin());
	}

	//CSV stats need to be written per frame (only send if we're running from the gamethread ticker, not the startup debug thread)
	if (IsInGameThread())
	{
		CSV_CUSTOM_STAT_DEFINED(ThroughputMbps, Throughput, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_DEFINED(MaxReqThroughputMbps, MaxReqThroughput, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_DEFINED(MinReqThroughputMbps, MinReqThroughput, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_DEFINED(RequestCountPerSec, (int32)ReqCount, ECsvCustomStatOp::Set);
	}

	return true;
}



void FStorageServerConnectionDebug::OnDraw(UCanvas* Canvas, APlayerController*)
{
	FScopeLock Lock(&CS);

	static constexpr float  ViewXRel = 0.2f;
	static constexpr float  ViewYRel = 0.12f;
	static constexpr float  ViewWidthRel = 0.4f;
	static constexpr float  ViewHeightRel = 0.18f;
	static constexpr double TextHeight = 16.0;
	static constexpr double MaxHeightScaleThroughput = 6000;
	static constexpr double MaxHeightScaleRequest = 5000;
	static constexpr int	LineThickness = 3;
	static double			HeightScaleThroughput = MaxHeightScaleThroughput;
	static double			HeightScaleRequest = MaxHeightScaleRequest;

	if (GZenShowGraphs)
	{
		double StatsTimeNow = FPlatformTime::Seconds();

		int ViewX = (int)(ViewXRel * Canvas->ClipX);
		int ViewY = (int)(ViewYRel * Canvas->ClipY);
		int ViewWidth = (int)(ViewWidthRel * Canvas->ClipX);;
		int ViewHeight = (int)(ViewHeightRel * Canvas->ClipY);;
		double PixelsPerSecond = ViewWidth / WidthSeconds;

		auto DrawLine =
			[Canvas](double X0, double Y0, double X1, double Y1, const FLinearColor& Color, double Thickness)
			{
				FCanvasLineItem Line{ FVector2D{X0, Y0}, FVector2D{X1, Y1} };
				Line.SetColor(Color);
				Line.LineThickness = Thickness;
				Canvas->DrawItem(Line);
			};

		auto DrawString =
			[Canvas](const FString& Str, int X, int Y, bool bCentre = true)
			{
				FCanvasTextItem Text{ FVector2D(X, Y), FText::FromString(Str), GEngine->GetTinyFont(), FLinearColor::Yellow };
				Text.EnableShadow(FLinearColor::Black);
				Text.bCentreX = bCentre;
				Text.bCentreY = bCentre;
				Canvas->DrawItem(Text);
			};

		double MaxValueInHistory = 0.0;

		if (History.size())
		{
			ViewY += TextHeight;
			DrawString(FString::Printf(TEXT("Request Throughput MIN/MAX: [%.2f] / [%.2f] Mbps"), History[History.size()-1].MinRequestThroughput, History[History.size()-1].MaxRequestThroughput), ViewX, ViewY, false);
			ViewY += TextHeight;
		}

		//FIRST GRAPH
		MaxValueInHistory = 0.0;
		double HeightScale = HeightScaleThroughput;
		ViewY += TextHeight;
		{ // draw graph edges + label
			const FLinearColor Color = FLinearColor::White;
			DrawLine(ViewX, ViewY + ViewHeight, ViewX + ViewWidth, ViewY + ViewHeight, Color, 1);
			DrawLine(ViewX, ViewY, ViewX, ViewY + ViewHeight, Color, 1);
			DrawLine(ViewX + ViewWidth, ViewY, ViewX + ViewWidth, ViewY + ViewHeight, Color, 1);
			DrawString(TEXT("ZenServer Throughput Mbps"), ViewX, ViewY + ViewHeight + 10, false);
		}

		for (int I = History.size() - 1; I >= 0; --I)
		{
			const auto& Item = History[I];
			const double       X = ViewX + ViewWidth - PixelsPerSecond * (StatsTimeNow - Item.Time);
			const double       H = FMath::Min(ViewHeight, ViewHeight * (Item.Throughput / HeightScale));
			const double       Y = ViewY + ViewHeight - H;
			const FLinearColor Color = FLinearColor::Yellow;
			
			DrawLine(X, ViewY + ViewHeight - 1, X, Y, Color, LineThickness);
			DrawString(FString::Printf(TEXT("%.2f"), Item.Throughput), X, Y - 11);

			if (Item.Throughput > MaxValueInHistory)
				MaxValueInHistory = Item.Throughput;
		}
		HeightScaleThroughput = FMath::Min(MaxHeightScaleThroughput, FMath::Max(MaxValueInHistory, 1.0));

		//SECOND GRAPH
		MaxValueInHistory = 0.0;
		ViewY += ViewHeight + (TextHeight * 2) ;
		HeightScale = HeightScaleRequest;

		{ // draw graph edges + label
			const FLinearColor Color = FLinearColor::White;
			DrawLine(ViewX, ViewY + ViewHeight, ViewX + ViewWidth, ViewY + ViewHeight, Color, 1);
			DrawLine(ViewX, ViewY, ViewX, ViewY + ViewHeight, Color, 1);
			DrawLine(ViewX + ViewWidth, ViewY, ViewX + ViewWidth, ViewY + ViewHeight, Color, 1);
			DrawString(TEXT("ZenServer Request/Sec Count"), ViewX, ViewY + ViewHeight + 10, false);
		}

		for (int I = History.size() - 1; I >= 0; --I)
		{
			const auto& Item = History[I];
			const double       X = ViewX + ViewWidth - PixelsPerSecond * (StatsTimeNow - Item.Time);
			const double       H = FMath::Min(ViewHeight, ViewHeight * (Item.RequestCount / HeightScale));
			const double       Y = ViewY + ViewHeight - H;
			const FLinearColor Color = FLinearColor::Gray;

			DrawLine(X, ViewY + ViewHeight - 1, X, Y, Color, LineThickness);
			DrawString(FString::Printf(TEXT("%.d"), Item.RequestCount), X, Y - 11);

			if (Item.RequestCount > MaxValueInHistory)
				MaxValueInHistory = Item.RequestCount;
		}
		HeightScaleRequest = FMath::Min(MaxHeightScaleRequest, FMath::Max(MaxValueInHistory, 1.0));
	}

	if (GZenShowIndicator)
	{
		static bool SettingsLoaded = false;
		if (SettingsLoaded == false)
		{
			LoadZenStreamingSettings();
			SettingsLoaded = true;
		}
		
		DrawZenIndicator(Canvas);
	}
	else if (bDestroyZenIcon)
	{
		DestroyZenIcon();
	}
}

void FStorageServerConnectionDebug::LoadZenStreamingSettings()
{
	float Val = 0.f;
	bool bShowIndicator = true;
	bShowIndicator = GConfig->GetBoolOrDefault(TEXT("/Script/StorageServerClient.ZenStreamingSettings"), TEXT("zen.indicator.show"), GZenShowIndicator, GGameIni);
	if (bShowIndicator != GZenShowIndicator)
	{
		CVarShowZenIndicator->Set(bShowIndicator, ECVF_SetByGameSetting);
	}

	Val = GConfig->GetFloatOrDefault(TEXT("/Script/StorageServerClient.ZenStreamingSettings"), TEXT("zen.indicator.x"), GZenIndicatorPosX, GGameIni);
	if (Val != GZenIndicatorPosX)
	{
		CVarZenIndicatorPosX->Set(Val, ECVF_SetByGameSetting);
	}

	Val = GConfig->GetFloatOrDefault(TEXT("/Script/StorageServerClient.ZenStreamingSettings"), TEXT("zen.indicator.y"), GZenIndicatorPosY, GGameIni);
	if (Val != GZenIndicatorPosY)
	{
		CVarZenIndicatorPosY->Set(Val, ECVF_SetByGameSetting);
	}
	
	Val = GConfig->GetFloatOrDefault(TEXT("/Script/StorageServerClient.ZenStreamingSettings"), TEXT("zen.indicator.fadetime"), GZenIndicatorFadeTime, GGameIni);
	if (Val != GZenIndicatorFadeTime)
	{
		CVarZenIndicatorFadeTime->Set(Val, ECVF_SetByGameSetting);
	}
	
	Val = GConfig->GetFloatOrDefault(TEXT("/Script/StorageServerClient.ZenStreamingSettings"), TEXT("zen.indicator.alpha"), GZenIndicatorAlpha, GGameIni);
	if (Val != GZenIndicatorAlpha)
	{
		CVarZenIndicatorAlpha->Set(Val, ECVF_SetByGameSetting);
	}

	CVarShowZenIndicator->SetOnChangedCallback(FConsoleVariableDelegate::CreateLambda([this](IConsoleVariable* Var)
		{
			IndicatorElapsedTime = 0.f;
			if (!Var->GetBool() && ZenIcon != nullptr)
			{
				bDestroyZenIcon = true;
			}
		}
	));

	CVarZenIndicatorFadeTime->SetOnChangedCallback(FConsoleVariableDelegate::CreateLambda([this](IConsoleVariable* Var)
		{
			IndicatorElapsedTime = 0.f;
		}
	));
}

void FStorageServerConnectionDebug::DrawZenIndicator(UCanvas* Canvas)
{
	if (ZenIcon == nullptr)
	{
		CreateZenIcon();
	}

	if (ZenIcon)
	{
		FCanvasIcon CanvasIcon = UCanvas::MakeIcon(ZenIcon);
		UFont* StringFont = GEngine->GetSmallFont();

		float Alpha = FMath::Clamp(GZenIndicatorAlpha, 0.0f, 1.0f);
		const float DeltaTime = FMath::Clamp(IndicatorElapsedTime - GZenIndicatorFadeTime, 0.f, IndicatorFadeSpeed);
		const float AlphaDelta = GZenIndicatorAlpha / IndicatorFadeSpeed;
		if (GZenIndicatorFadeTime > 0 && DeltaTime > 0)
		{
			Alpha = GZenIndicatorAlpha - DeltaTime * AlphaDelta;
		}

		const float IconPosX = (Canvas->ClipX - ZenIcon->GetSurfaceWidth()) * GZenIndicatorPosX;
		const float IconPosY = (Canvas->ClipY - ZenIcon->GetSurfaceHeight()) * GZenIndicatorPosY;
		const float BackgroundPosX = IconPosX - ZenIconPadding;
		const float BackgroundPosY = IconPosY - ZenIconPadding;
		const float BackgroundSizeX = ZenIcon->GetSurfaceWidth() + 2 * ZenIconPadding + ZenIndicatorTextWidth;
		const float BackgroundSizeY = ZenIcon->GetSurfaceHeight() + 2 * ZenIconPadding;
		const float IconScreenPosX = IconPosX / Canvas->ClipX;
		const float IconScreenPosY = IconPosY / Canvas->ClipY;
		const float TextPosX = IconPosX + ZenIcon->GetSurfaceWidth() + 2 * ZenIconPadding;
		const float TextPosY = IconPosY + ZenIconPadding;

		const FLinearColor BackgroundColor(0.0f, 0.0f, 0.0f, Alpha);
		const FLinearColor TextBackgroundColor(0.7f, 0.7f, 0.7f, Alpha);
		const FLinearColor DebugBackgroundColor(1.f, 0.f, 0.f, Alpha);

		FCanvasTileItem BackgroundTileItem(
			FVector2D(BackgroundPosX, BackgroundPosY),
			FVector2D(BackgroundSizeX, BackgroundSizeY),
			BackgroundColor);

		FCanvasTileItem DebugBackgroundTile(
			FVector2D(IconPosX, IconPosY),
			FVector2D(ZenIcon->GetSurfaceWidth(), ZenIcon->GetSurfaceHeight()),
			DebugBackgroundColor);

		DebugBackgroundTile.BlendMode = SE_BLEND_AlphaBlend;
		BackgroundTileItem.BlendMode = SE_BLEND_AlphaBlend;

		Canvas->DrawItem(BackgroundTileItem);
		Canvas->SetDrawColor(255, 255, 255, (uint8)(Alpha * 255.f));
		Canvas->DrawIcon(CanvasIcon, IconPosX, IconPosY, 1.f);
		FString ZenStreamingString = FString::Printf(TEXT("Zen %s from: %s"), UE::IsUsingZenPakFileStreaming() ? TEXT("pak streaming") : TEXT("streaming"), * HostAddress);
		Canvas->Canvas->DrawShadowedString(TextPosX, TextPosY, *ZenStreamingString, StringFont, TextBackgroundColor);
		FString ZenBandwidthString = FString::Printf(TEXT("Bandwidth: %.2f Mbps"), Throughput);
		Canvas->Canvas->DrawShadowedString(TextPosX, TextPosY + StringFont->GetMaxCharHeight() + 8, *ZenBandwidthString, StringFont, TextBackgroundColor);

		if (Alpha == 0.f)
		{
			bDestroyZenIcon = true;
			GZenShowIndicator = false;
		}
	}
}

void FStorageServerConnectionDebug::CreateZenIcon()
{
	if (ZenIcon == nullptr && GEngine->DefaultZenStreamingTextureName.IsValid())
	{
		ZenIcon = Cast<UTexture2D>(GEngine->DefaultZenStreamingTextureName.TryLoad());
		if (ZenIcon != nullptr)
		{
			ZenIcon->AddToRoot();
		}
	}
}

void FStorageServerConnectionDebug::DestroyZenIcon()
{
	if (ZenIcon != nullptr)
	{
		ZenIcon->RemoveFromRoot();
		ZenIcon = nullptr;
		bDestroyZenIcon = false;
	}
}

class FStorageServerClientDebugModule
	: public IModuleInterface
	, public FRunnable
{
public:
	virtual void StartupModule() override
	{
		if (IStorageServerPlatformFile* StorageServerPlatformFile = IStorageServerClientModule::FindStorageServerPlatformFile())
		{
			ConnectionDebug.Reset( new FStorageServerConnectionDebug(StorageServerPlatformFile) );
			OnDrawDebugHandle = UDebugDrawService::Register(TEXT("Game"), FDebugDrawDelegate::CreateRaw(ConnectionDebug.Get(), &FStorageServerConnectionDebug::OnDraw) );

			// start by capturing engine initialization stats on a background thread
			StartThread();

			// once the engine has initialized, switch to a more lightweight gamethread ticker
			FCoreDelegates::GetOnPostEngineInit().AddLambda([this]
			{
				StopThread();
				StartTick();
			});

			// load the low-level network tracing module too, so we get platform bandwidth stats as well
			if (FModuleManager::Get().ModuleExists(TEXT("LowLevelNetTrace")))
			{
				FModuleManager::Get().LoadModule(TEXT("LowLevelNetTrace"));
			}
		}
	}

	virtual void ShutdownModule() override
	{
		if (ConnectionDebug.IsValid())
		{
			StopThread();
			StopTick();
			UDebugDrawService::Unregister(OnDrawDebugHandle);
			ConnectionDebug.Reset();
		}
	}

	void StartThread()
	{
		check(!Thread.IsValid());
		ThreadStopEvent = FPlatformProcess::GetSynchEventFromPool(true);
		Thread.Reset( FRunnableThread::Create(this, TEXT("StorageServerStartupDebug"), 0, TPri_Lowest) );
	}

	void StopThread()
	{
		if (Thread.IsValid())
		{
			Thread.Reset();
			FPlatformProcess::ReturnSynchEventToPool(ThreadStopEvent);
			ThreadStopEvent = nullptr;
		}
	}

	void StartTick()
	{
		check(!TickHandle.IsValid());
		TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(ConnectionDebug.Get(), &FStorageServerConnectionDebug::OnTick));
	}

	void StopTick()
	{
		if (TickHandle.IsValid())
		{
			FTSTicker::RemoveTicker(TickHandle);
			TickHandle.Reset();
		}
	}

	// FRunnable interface
	virtual uint32 Run() override
	{
		while(!ThreadStopEvent->Wait(10))
		{
			ConnectionDebug->OnTick(0);
		}
		return 0;
	}

	virtual void Stop() override
	{
		ThreadStopEvent->Trigger();
	}
	// end of FRunnable interface


	TUniquePtr<FStorageServerConnectionDebug> ConnectionDebug;
	FDelegateHandle OnDrawDebugHandle;

	TUniquePtr<FRunnableThread> Thread;
	FEvent* ThreadStopEvent = nullptr;

	FTSTicker::FDelegateHandle TickHandle;

};

IMPLEMENT_MODULE(FStorageServerClientDebugModule, StorageServerClientDebug);

#else

// shipping stub
IMPLEMENT_MODULE(FDefaultModuleImpl, StorageServerClientDebug);

#endif // !UE_BUILD_SHIPPING



