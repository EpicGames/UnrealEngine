// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Canvas.h"
#include "HAL/CriticalSection.h"
#include <vector>
#include "IStorageServerPlatformFile.h"

#if !UE_BUILD_SHIPPING
class FStorageServerConnectionDebug
{
public:
	FStorageServerConnectionDebug( IStorageServerPlatformFile* InStorageServerPlatformFile )
		: StorageServerPlatformFile(InStorageServerPlatformFile)
		, HostAddress(InStorageServerPlatformFile->GetHostAddr())
	{
		IndicatorLastTime = FPlatformTime::Seconds();
	}

	bool OnTick(float); // FTickerDelegate
	void OnDraw(UCanvas*, APlayerController*); // FDebugDrawDelegate

private:
	void LoadZenStreamingSettings();

	void DrawZenIndicator(UCanvas* Canvas);
	void CreateZenIcon();
	void DestroyZenIcon();

	double MaxReqThroughput = 0.0;
	double MinReqThroughput = 0.0;
	uint32 ReqCount = 0;
	double Throughput = 0.0;

	struct HistoryItem
	{
		double Time;
		double MaxRequestThroughput;
		double MinRequestThroughput;
		double Throughput;
		uint32 RequestCount;
	};

	std::vector<HistoryItem> History = {{0, 0, 0, 0, 0}};

	static constexpr float UpdateStatsTimer = 1.0;
	double UpdateStatsTime = 0.0;

	IStorageServerPlatformFile* StorageServerPlatformFile = nullptr;
	FString HostAddress;

	FCriticalSection CS;

	class UTexture2D* ZenIcon = nullptr;
	bool bDestroyZenIcon = false;
	double IndicatorElapsedTime = 0.0;
	double IndicatorLastTime = 0.0;
};
#endif // !UE_BUILD_SHIPPING
