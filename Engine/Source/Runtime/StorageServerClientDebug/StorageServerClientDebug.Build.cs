// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class StorageServerClientDebug : ModuleRules
{
	public StorageServerClientDebug(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",		// For on-screen debug messages
				"Sockets",
				"StorageServerClient",
				"Json"
			}
		);
	}
}
